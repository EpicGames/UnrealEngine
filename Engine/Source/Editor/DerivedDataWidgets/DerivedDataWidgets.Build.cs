// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class DerivedDataWidgets : ModuleRules
{
	public DerivedDataWidgets(ReadOnlyTargetRules Target)
		 : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"ToolMenus",
				"OutputLog",
				"DerivedDataCache",
				"EditorSubsystem",
				"WorkspaceMenuStructure",
				"MessageLog",
				"ToolWidgets"
			});

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
			});


		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			});
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}


