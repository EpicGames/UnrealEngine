// Copyright Epic Games, Inc. All Rights Reserved.

#include "DerivedDataInformation.h"
#include "DerivedDataCacheUsageStats.h"
#include "DerivedDataCacheInterface.h"
#include "Settings/EditorProjectSettings.h"
#include "Settings/EditorSettings.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "DerivedDataEditor"

double FDerivedDataInformation::LastGetTime = 0;
double FDerivedDataInformation::LastPutTime = 0;
bool FDerivedDataInformation::bIsDownloading = false;
bool FDerivedDataInformation::bIsUploading = false;
FText FDerivedDataInformation::RemoteCacheWarningMessage;
ERemoteCacheState FDerivedDataInformation::RemoteCacheState= ERemoteCacheState::Unavailable;

static TArray<TSharedRef<const FDerivedDataCacheStatsNode>> GetCacheUsageStats()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	TSharedRef<FDerivedDataCacheStatsNode> RootUsage = GetDerivedDataCacheRef().GatherUsageStats();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	TArray<TSharedRef<const FDerivedDataCacheStatsNode>> LeafUsageStats;
	RootUsage->ForEachDescendant([&LeafUsageStats](TSharedRef<const FDerivedDataCacheStatsNode> Node)
	{
		if (Node->Children.IsEmpty())
		{
			LeafUsageStats.Add(Node);
		}
	});
	return LeafUsageStats;
}

double FDerivedDataInformation::GetCacheActivitySizeBytes(bool bGet, bool bLocal)
{
	int64 TotalBytes = 0;

#if ENABLE_COOK_STATS
	for (const TSharedRef<const FDerivedDataCacheStatsNode>& Usage : GetCacheUsageStats())
	{
		if (Usage->IsLocal() != bLocal)
		{
			continue;
		}

		for (const auto& KVP : Usage->UsageStats)
		{
			const FDerivedDataCacheUsageStats& Stats = KVP.Value;

			if (bGet)
			{
				TotalBytes += Stats.GetStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
			}
			else
			{
				TotalBytes += Stats.PutStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
			}
		}
	}
#endif // ENABLE_COOK_STATS

	return (double)TotalBytes;
}


double FDerivedDataInformation::GetCacheActivityTimeSeconds(bool bGet, bool bLocal)
{
	int64 TotalCycles = 0;

#if ENABLE_COOK_STATS
	for (const TSharedRef<const FDerivedDataCacheStatsNode>& Usage : GetCacheUsageStats())
	{
		if (Usage->IsLocal() != bLocal)
		{
			continue;
		}

		for (const auto& KVP : Usage->UsageStats)
		{
			const FDerivedDataCacheUsageStats& Stats = KVP.Value;

			if (bGet)
			{
				TotalCycles +=
					(Stats.GetStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles) +
						Stats.GetStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Miss, FCookStats::CallStats::EStatType::Cycles));

				TotalCycles +=
					(Stats.PrefetchStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles) +
						Stats.PrefetchStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Miss, FCookStats::CallStats::EStatType::Cycles));
			}
			else
			{
				TotalCycles +=
					(Stats.PutStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles) +
						Stats.PutStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Miss, FCookStats::CallStats::EStatType::Cycles));
			}
		}
	}
#endif // ENABLE_COOK_STATS

	return (double)TotalCycles * FPlatformTime::GetSecondsPerCycle();
}

bool FDerivedDataInformation::GetHasRemoteCache()
{
	for (const TSharedRef<const FDerivedDataCacheStatsNode>& Usage : GetCacheUsageStats())
	{
		if (!Usage->IsLocal())
		{
				return true;
		}
	}
	return false;
}

bool FDerivedDataInformation::GetHasZenCache()
{
	for (const TSharedRef<const FDerivedDataCacheStatsNode>& Usage : GetCacheUsageStats())
	{
		if (Usage->GetCacheType().Equals(TEXT("Zen")))
		{
				return true;
		}
	}
	return false;
}

bool FDerivedDataInformation::GetHasUnrealCloudCache()
{
	for (const TSharedRef<const FDerivedDataCacheStatsNode>& Usage : GetCacheUsageStats())
	{
		if (Usage->GetCacheType().Equals(TEXT("Unreal Cloud DDC")))
		{
			return true;
		}
	}
	return false;
}

void FDerivedDataInformation::UpdateRemoteCacheState()
{
	RemoteCacheState = ERemoteCacheState::Unavailable;

	if ( GetHasRemoteCache() )
	{
		const double OldLastGetTime = LastGetTime;
		const double OldLastPutTime = LastPutTime;

		LastGetTime = FDerivedDataInformation::GetCacheActivityTimeSeconds(/*bGet*/ true, /*bLocal*/ false);
		LastPutTime = FDerivedDataInformation::GetCacheActivityTimeSeconds(/*bGet*/ false, /*bLocal*/ false);

		if (OldLastGetTime != 0.0 && OldLastPutTime != 0.0)
		{
			bIsDownloading = OldLastGetTime != LastGetTime;
			bIsUploading = OldLastPutTime != LastPutTime;
		}

		if (bIsUploading || bIsDownloading)
		{
			RemoteCacheState = ERemoteCacheState::Busy;
		}
		else
		{
			RemoteCacheState = ERemoteCacheState::Idle;
		}
	}

	if (const UDDCProjectSettings* DDCProjectSettings = GetDefault<UDDCProjectSettings>(); DDCProjectSettings && DDCProjectSettings->EnableWarnings)
	{
		const UEditorSettings* EditorSettings = GetDefault<UEditorSettings>();

		if (EditorSettings && EditorSettings->bEnableDDCNotifications)
		{
			if (DDCProjectSettings->RecommendEveryoneUseUnrealCloudDDC && EditorSettings->bNotifyUseUnrealCloudDDC && !GetHasUnrealCloudCache())
			{
				RemoteCacheState = ERemoteCacheState::Warning;
				RemoteCacheWarningMessage = FText(LOCTEXT("UnrealCloudDDCWarning", "It is recommended that you use Unreal Cloud DDC.\nDisable this notification in the settings."));
			}
			else if (DDCProjectSettings->RecommendEveryoneSetupAGlobalLocalDDCPath && EditorSettings->bNotifySetupDDCPath && EditorSettings->GlobalLocalDDCPath.Path.IsEmpty())
			{
				RemoteCacheState = ERemoteCacheState::Warning;
				RemoteCacheWarningMessage = FText(LOCTEXT("GlobalLocalDDCPathWarning", "It is recommended that you set up a valid Local DDC Path.\nDisable this notification or set up a valid Local DDC Path in the settings."));
			}
			else if (DDCProjectSettings->RecommendEveryoneSetupAGlobalSharedDDCPath && EditorSettings->bNotifySetupDDCPath && EditorSettings->GlobalSharedDDCPath.Path.IsEmpty())
			{
				RemoteCacheState = ERemoteCacheState::Warning;
				RemoteCacheWarningMessage = FText(LOCTEXT("GlobalSharedDDCPathWarning", "It is recommended that you set up a valid Shared DDC Path.\nDisable this notification or set up a valid Shared DDC Path in the settings."));
			}
			else if (DDCProjectSettings->RecommendEveryoneEnableS3DDC && EditorSettings->bNotifyEnableS3DD && !EditorSettings->bEnableS3DDC)
			{
				RemoteCacheState = ERemoteCacheState::Warning;
				RemoteCacheWarningMessage = FText(LOCTEXT("AWSS3CacheEnabledWarning", "It is recommended that you enable the AWS S3 Cache.\nDisable this notification or enable the AWS S3 Cache in the settings."));
			}
			else if (DDCProjectSettings->RecommendEveryoneSetupAGlobalS3DDCPath && EditorSettings->bNotifySetupDDCPath && EditorSettings->GlobalS3DDCPath.Path.IsEmpty())
			{
				RemoteCacheState = ERemoteCacheState::Warning;
				RemoteCacheWarningMessage = FText(LOCTEXT("S3GloblaLocalPathWarning", "It is recommended that you set up a valid Local S3 DDC Path.\nDisable this notification or set up a valid Local S3 DDC Path in the settings."));
			}
		}
	}
}


FText FDerivedDataInformation::GetRemoteCacheStateAsText()
{
	switch (FDerivedDataInformation::GetRemoteCacheState())
	{
	case ERemoteCacheState::Idle:
	{
		return FText(LOCTEXT("DDCStateIdle","Idle"));
	}

	case ERemoteCacheState::Busy:
	{
		return FText(LOCTEXT("DDCStateBusy", "Busy"));
	}

	case ERemoteCacheState::Unavailable:
	{
		return FText(LOCTEXT("DDCStateUnavailable", "Unavailable"));
	}

	default:
	case ERemoteCacheState::Warning:
	{
		return FText(LOCTEXT("DDCStateWarning", "Warning"));
	}
	}
}


#undef LOCTEXT_NAMESPACE