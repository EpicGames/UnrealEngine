// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDerivedDataDialogs.h"
#include "Algo/Sort.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheUsageStats.h"
#include "DerivedDataInformation.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Misc/ExpressionParser.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DerivedDataCacheEditor"

template <typename ValueType>
static FString ZeroDecimalFormat(ValueType Value)
{
	const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
		.SetUseGrouping(true)
		.SetMinimumFractionalDigits(0)
		.SetMaximumFractionalDigits(0);
	return FastDecimalFormat::NumberToString(Value, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
}

static FString SingleDecimalFormat(double Value)
{
	const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
		.SetUseGrouping(true)
		.SetMinimumFractionalDigits(1)
		.SetMaximumFractionalDigits(1);
	return FastDecimalFormat::NumberToString(Value, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
}

void SDerivedDataRemoteStoreDialog::Construct(const FArguments& InArgs)
{
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 20, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];

	RegisterActiveTimer(0.5f, FWidgetActiveTimerDelegate::CreateSP(this, &SDerivedDataRemoteStoreDialog::UpdateGridPanels));
}

EActiveTimerReturnType SDerivedDataRemoteStoreDialog::UpdateGridPanels(double InCurrentTime, float InDeltaTime)
{
	(*GridSlot)
	[
		GetGridPanel()
	];

	SlatePrepass(GetPrepassLayoutScaleMultiplier());

	return EActiveTimerReturnType::Continue;
}

TSharedRef<SWidget> SDerivedDataRemoteStoreDialog::GetGridPanel()
{
	// Gather the latest resource stats
	TArray<FDerivedDataCacheResourceStat> DDCResourceStats;
	GatherDerivedDataCacheResourceStats(DDCResourceStats);

	FDerivedDataCacheResourceStat DDCResourceStatsTotal(TEXT("Total"));

	// Find the total
	for (const FDerivedDataCacheResourceStat& Stat : DDCResourceStats)
	{
		if (Stat.AssetType == TEXT("Total"))
		{
			DDCResourceStatsTotal = Stat;
			break;
		}
	}

	const double DownloadedBytesMB = FUnitConversion::Convert(FDerivedDataInformation::GetCacheActivitySizeBytes(true, false), EUnit::Bytes, EUnit::Megabytes);
	const double UploadedBytesMB = FUnitConversion::Convert(FDerivedDataInformation::GetCacheActivitySizeBytes(false, false), EUnit::Bytes, EUnit::Megabytes);

	TSharedRef<SGridPanel> Panel =
		SNew(SGridPanel);

	int32 Row = 0;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
		.ColorAndOpacity(FStyleColors::Foreground)
		.Text(LOCTEXT("Remote Storage", "Remote Storage"))
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Connected", "Connected"))
	];

	Panel->AddSlot(1, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Text(FDerivedDataInformation::GetHasRemoteCache() ? LOCTEXT("True", "True") : LOCTEXT("False", "False"))
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Downloaded", "Downloaded"))
	];

	Panel->AddSlot(1, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Text(FText::FromString(SingleDecimalFormat(DownloadedBytesMB) + TEXT(" MiB")))
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Uploaded", "Uploaded"))
	];

	Panel->AddSlot(1, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Text(FText::FromString(SingleDecimalFormat(UploadedBytesMB) + TEXT(" MiB")))
	];

	Row++;

	return Panel;
}



void SDerivedDataResourceUsageDialog::Construct(const FArguments& InArgs)
{
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 20, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];

	RegisterActiveTimer(0.5f, FWidgetActiveTimerDelegate::CreateSP(this, &SDerivedDataResourceUsageDialog::UpdateGridPanels));
}

EActiveTimerReturnType SDerivedDataResourceUsageDialog::UpdateGridPanels(double InCurrentTime, float InDeltaTime)
{
	(*GridSlot)
	[
		GetGridPanel()
	];

	SlatePrepass(GetPrepassLayoutScaleMultiplier());

	return EActiveTimerReturnType::Continue;
}

TSharedRef<SWidget> SDerivedDataResourceUsageDialog::GetGridPanel()
{
	// Gather the latest resource stats
	TArray<FDerivedDataCacheResourceStat> DDCResourceStats;
	GatherDerivedDataCacheResourceStats(DDCResourceStats);

	// Sort results on descending build count, then descending load size, then ascending asset type.
	const auto CompareStats = [](const FDerivedDataCacheResourceStat& A, const FDerivedDataCacheResourceStat& B)
	{
		if (A.BuildCount != B.BuildCount)
		{
			return A.BuildCount > B.BuildCount;
		}
		if (A.LoadSizeMB != B.LoadSizeMB)
		{
			return A.LoadSizeMB > B.LoadSizeMB;
		}
		return A.AssetType.Compare(B.AssetType, ESearchCase::IgnoreCase) < 0;
	};
	Algo::Sort(DDCResourceStats, CompareStats);

	FDerivedDataCacheResourceStat DDCResourceStatsTotal(TEXT("Total"));

	// Find the total and store it off, we will render that out separately
	for (const FDerivedDataCacheResourceStat& Stat : DDCResourceStats)
	{
		if (Stat.AssetType == TEXT("Total"))
		{
			DDCResourceStatsTotal = Stat;
			break;
		}
	}

	TSharedRef<SGridPanel> Panel =
		SNew(SGridPanel);

	int32 Row = 0;

	const float RowMargin = 0.0f;
	const float ColumnMargin = 10.0f;
	const FMargin TitleMargin(0.0f, 10.0f, ColumnMargin, 10.0f);
	const FMargin TitleMarginFirstColumn(ColumnMargin, 10.0f);
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
	const FMargin DefaultMargin(0.0f, RowMargin, ColumnMargin, RowMargin);
	const FMargin DefaultMarginFirstColumn(ColumnMargin, RowMargin);

	Panel->AddSlot(3, Row)
	.HAlign(HAlign_Center)
	[
		SNew(STextBlock)
		.Margin(DefaultMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Loaded", "Loaded"))
	];

	Panel->AddSlot(6, Row)
	.HAlign(HAlign_Center)
	[
		SNew(STextBlock)
		.Margin(DefaultMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Built", "Built"))
	];

	Row++;

	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(TitleMarginFirstColumn)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("ResourceType", "Resource Type"))
	];

	Panel->AddSlot(1, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
	.ColorAndOpacity(TitleColor)
	.Font(TitleFont)
	.Text(LOCTEXT("Hit", "Hit%"))
	];

	Panel->AddSlot(2, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Count", "Count"))
	];

	Panel->AddSlot(3, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Time (Sec)", "Time (Sec)"))
	];

	Panel->AddSlot(4, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Size (MiB)", "Size (MiB)"))
	];

	Panel->AddSlot(5, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Count", "Count"))
	];

	Panel->AddSlot(6, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Time (Sec)", "Time (Sec)"))
	];

	Panel->AddSlot(7, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Size (MiB)", "Size (MiB)"))
	];

	Row++;

	for (const FDerivedDataCacheResourceStat& Stat : DDCResourceStats)
	{	
		if (Stat.AssetType == TEXT("Total"))
		{
			continue;
		}

		Panel->AddSlot(0, Row)
		[
			SNew(STextBlock)
			.Margin(DefaultMarginFirstColumn)
			.Text(FText::FromString(Stat.AssetType))
		];

		Panel->AddSlot(1, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(SingleDecimalFormat(Stat.Efficiency*100.0)))
		];


		Panel->AddSlot(2, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(ZeroDecimalFormat(Stat.LoadCount)))
		];

		Panel->AddSlot(3, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(SingleDecimalFormat(Stat.LoadTimeSec)))
		];

		Panel->AddSlot(4, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(SingleDecimalFormat(Stat.LoadSizeMB)))
		];

		Panel->AddSlot(5, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(ZeroDecimalFormat(Stat.BuildCount)))
		];

		Panel->AddSlot(6, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(SingleDecimalFormat(Stat.BuildTimeSec)))
		];

		Panel->AddSlot(7, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(SingleDecimalFormat(Stat.BuildSizeMB)))
		];

		Row++;
	}
	
	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
		.Margin(TitleMarginFirstColumn)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(DDCResourceStatsTotal.AssetType))
	];

	Panel->AddSlot(1, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(SingleDecimalFormat(DDCResourceStatsTotal.Efficiency * 100.0)))
	];


	Panel->AddSlot(2, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(ZeroDecimalFormat(DDCResourceStatsTotal.LoadCount)))
	];

	Panel->AddSlot(3, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(SingleDecimalFormat(DDCResourceStatsTotal.LoadTimeSec)))
	];

	Panel->AddSlot(4, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(SingleDecimalFormat(DDCResourceStatsTotal.LoadSizeMB)))
	];

	Panel->AddSlot(5, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(ZeroDecimalFormat(DDCResourceStatsTotal.BuildCount)))
	];

	Panel->AddSlot(6, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(SingleDecimalFormat(DDCResourceStatsTotal.BuildTimeSec)))
	];

	Panel->AddSlot(7, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(SingleDecimalFormat(DDCResourceStatsTotal.BuildSizeMB)))
	];

	return Panel;
}



void SDerivedDataCacheStatisticsDialog::Construct(const FArguments& InArgs)
{
	const float RowMargin = 0.0f;
	const float TitleMargin = 10.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0, 20, 0, 0)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Margin(TitleMargin)
				.ColorAndOpacity(TitleColor)
				.Font(TitleFont)
				.Justification(ETextJustify::Left)
				.Text(FText::FromString(GetDerivedDataCache()->GetGraphName()))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 5, 0, 0)
		.Expose(GridSlot)
		[
			GetGridPanel()
		]
	];

	RegisterActiveTimer(0.5f, FWidgetActiveTimerDelegate::CreateSP(this, &SDerivedDataCacheStatisticsDialog::UpdateGridPanels));
}

EActiveTimerReturnType SDerivedDataCacheStatisticsDialog::UpdateGridPanels(double InCurrentTime, float InDeltaTime)
{
	(*GridSlot)
	[
		GetGridPanel()
	];

	SlatePrepass(GetPrepassLayoutScaleMultiplier());

	return EActiveTimerReturnType::Continue;
}



TSharedRef<SWidget> SDerivedDataCacheStatisticsDialog::GetGridPanel()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	TSharedRef<FDerivedDataCacheStatsNode> RootUsage = GetDerivedDataCache()->GatherUsageStats();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	TArray<TSharedRef<const FDerivedDataCacheStatsNode>> LeafUsageStats;
	RootUsage->ForEachDescendant([&LeafUsageStats](TSharedRef<const FDerivedDataCacheStatsNode> Node)
	{
		if (Node->Children.IsEmpty())
		{
			LeafUsageStats.Add(Node);
		}
	});

	TSharedRef<SGridPanel> Panel =
		SNew(SGridPanel);

	const float RowMargin = 0.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
	const FMargin DefaultMarginFirstColumn(ColumnMargin, RowMargin);

#if ENABLE_COOK_STATS

	int32 Row = 0;

	const FMargin TitleMargin(0.0f, 10.0f, ColumnMargin, 10.0f);
	const FMargin TitleMarginFirstColumn(ColumnMargin, 10.0f);
	const FMargin DefaultMargin(0.0f, RowMargin, ColumnMargin, RowMargin);
	
	Panel->AddSlot(0, Row)
	[
		SNew(STextBlock)
	];

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(TitleMarginFirstColumn)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
		.ColorAndOpacity(TitleColor)
		.Text(LOCTEXT("CacheType", "Cache Type"))
	];

	Panel->AddSlot(2, Row)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Location", "Location"))
	];

	Panel->AddSlot(3, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("HitPercentage", "Hit%"))
	];

	Panel->AddSlot(4, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Read", "Read (MiB)"))
		.AutoWrapText(true)
		.WrapTextAt(66.0f)
	];

	Panel->AddSlot(5, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Write", "Write (MiB)"))
		.AutoWrapText(true)
		.WrapTextAt(66.0f)
	];

	Panel->AddSlot(6, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Latency", "Latency (ms)"))
		.AutoWrapText(true)
		.WrapTextAt(66.0f)
	];

	Panel->AddSlot(7, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Read Speed", "Read Speed (MiB/s)"))
		.AutoWrapText(true)
		.WrapTextAt(100.0f)
	];

	Panel->AddSlot(8, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Write Speed", "Write Speed (MiB/s)"))
		.AutoWrapText(true)
		.WrapTextAt(100.0f)
	];

	Panel->AddSlot(9, Row)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Storage Size", "Storage Size"))
	];

	Panel->AddSlot(10, Row)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Path", "Path"))
	];

	Row++;

	double SumTotalGetMB = 0.0;
	double SumTotalPutMB = 0.0;
	uint64 SumTotalPhysicalSize = ~0ull;

	for (TSharedRef<const FDerivedDataCacheStatsNode> Node : LeafUsageStats)
	{
		FDerivedDataCacheUsageStats Stats;

		for (const auto& KVP : Node->UsageStats)
		{
			Stats.Combine(KVP.Value);
		}

		const int64 TotalGetBytes = Stats.GetStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
		const int64 TotalPutBytes = Stats.PutStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);

		const int64 TotalGetHits = Stats.GetStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter);
		const int64 TotalGetMisses = Stats.GetStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Miss, FCookStats::CallStats::EStatType::Counter);
		const int64 TotalRequests = TotalGetHits + TotalGetMisses;
		const double HitRate = TotalRequests > 0 ? 100.0 * static_cast<double>(TotalGetHits) / static_cast<double>(TotalRequests) : 0.0;

		const double TotalGetMB = FUnitConversion::Convert(static_cast<double>(TotalGetBytes), EUnit::Bytes, EUnit::Megabytes);
		const double TotalPutMB = FUnitConversion::Convert(static_cast<double>(TotalPutBytes), EUnit::Bytes, EUnit::Megabytes);

		SumTotalGetMB += TotalGetMB;
		SumTotalPutMB += TotalPutMB;

		const uint64 TotalPhysicalSize = Node->GetTotalPhysicalSize();
		if (TotalPhysicalSize != ~0ull)
		{
			if (SumTotalPhysicalSize == ~0ull)
			{
				SumTotalPhysicalSize = TotalPhysicalSize;
			}
			else
			{
				SumTotalPhysicalSize += TotalPhysicalSize;
			}
		}

		TSharedPtr<SImage> StatusIcon;
		switch (Node->GetCacheStatus())
		{
			case EDerivedDataCacheStatus::Information:
				StatusIcon = SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Help"))
					.ToolTipText(FText::FromString(Node->GetCacheStatusText()));
			break;
			case EDerivedDataCacheStatus::Warning:
				StatusIcon = SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Warning"))
					.ToolTipText(FText::FromString(Node->GetCacheStatusText()));
			break;
			case EDerivedDataCacheStatus::Error:
				StatusIcon = SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Error"))
					.ToolTipText(FText::FromString(Node->GetCacheStatusText()));
			break;
			case EDerivedDataCacheStatus::Deactivation:
				StatusIcon = SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.X"))
					.ToolTipText(FText::FromString(Node->GetCacheStatusText()));
			break;

		}

		if (StatusIcon.IsValid())
		{
			Panel->AddSlot(0, Row)
			[
				SNew(SHorizontalBox)
					.ToolTipText(FText::FromString(Node->GetCacheStatusText()))
					// Status icon
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						StatusIcon.ToSharedRef()
					]
			];
		}
		else
		{
			Panel->AddSlot(0, Row)
			[
				SNew(SHorizontalBox)
			];
		}

		Panel->AddSlot(1, Row)
		[
			SNew(STextBlock)
			.Margin(DefaultMarginFirstColumn)
			.Text(FText::FromString(Node->GetCacheType()))
		];

		Panel->AddSlot(2, Row)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(Node->IsLocal() ? LOCTEXT("Local", "Local") : LOCTEXT("Remote", "Remote"))
		];

		Panel->AddSlot(3, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(SingleDecimalFormat(HitRate)))
		];

		Panel->AddSlot(4, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(SingleDecimalFormat(TotalGetMB)))
		];

		Panel->AddSlot(5, Row)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(SingleDecimalFormat(TotalPutMB)))
		];

		if (Node->SpeedStats.LatencyMS != 0.0)
		{
			Panel->AddSlot(6, Row)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Margin(DefaultMargin)
				.Text(FText::FromString(SingleDecimalFormat(Node->SpeedStats.LatencyMS)))
			];
		}

		if (Node->SpeedStats.ReadSpeedMBs != 0.0)
		{
			Panel->AddSlot(7, Row)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Margin(DefaultMargin)
				.Text(FText::FromString(SingleDecimalFormat(Node->SpeedStats.ReadSpeedMBs)))
			];
		}

		if (Node->SpeedStats.WriteSpeedMBs != 0.0)
		{
			Panel->AddSlot(8, Row)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Margin(DefaultMargin)
				.Text(FText::FromString(SingleDecimalFormat(Node->SpeedStats.WriteSpeedMBs)))
			];
		}

		Panel->AddSlot(9, Row)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(Node->GetTotalPhysicalSize() == ~0ull ? LOCTEXT("N/A", "N/A") : FText::AsMemory(Node->GetTotalPhysicalSize(), EMemoryUnitStandard::IEC))
		];

		Panel->AddSlot(10, Row)
		[
			SNew(STextBlock)
			.Margin(DefaultMargin)
			.Text(FText::FromString(Node->GetCacheName()))
		];

		Row++;
	}

	Panel->AddSlot(1, Row)
	[
		SNew(STextBlock)
		.Margin(TitleMarginFirstColumn)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(LOCTEXT("Total", "Total"))
	];

	Panel->AddSlot(4, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(SingleDecimalFormat(SumTotalGetMB)))
	];

	Panel->AddSlot(5, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(FText::FromString(SingleDecimalFormat(SumTotalPutMB)))
	];

	Panel->AddSlot(9, Row)
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Margin(TitleMargin)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Text(SumTotalPhysicalSize == ~0ull ? LOCTEXT("N/A", "N/A") : FText::AsMemory(SumTotalPhysicalSize, EMemoryUnitStandard::IEC))
	];

#else
	Panel->AddSlot(0, 0)
	[
		SNew(STextBlock)
		.Margin(DefaultMarginFirstColumn)
		.ColorAndOpacity(TitleColor)
		.Font(TitleFont)
		.Justification(ETextJustify::Center)
		.Text(LOCTEXT("Disabled", "Cooking Stats Are Disabled For This Project"))
	];

#endif // ENABLE_COOK_STATS

	return Panel;
}

#undef LOCTEXT_NAMESPACE
