// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Internationalization/Text.h"

#define UE_API DERIVEDDATAWIDGETS_API

enum class DERIVEDDATAWIDGETS_API ERemoteCacheState : uint8
{
	Idle,
	Busy,
	Unavailable,
	Warning,
};

class FDerivedDataInformation
{
public:

	static UE_API double				GetCacheActivityTimeSeconds(bool bGet, bool bLocal);
	static UE_API double				GetCacheActivitySizeBytes(bool bGet, bool bLocal);
	static UE_API bool					GetHasRemoteCache();
	static UE_API bool					GetHasZenCache();
	static UE_API bool					GetHasUnrealCloudCache();
	static ERemoteCacheState	GetRemoteCacheState() { return RemoteCacheState; }
	static UE_API FText				GetRemoteCacheStateAsText();
	static FText				GetRemoteCacheWarningMessage() { return RemoteCacheWarningMessage; }
	static UE_API void					UpdateRemoteCacheState();
	static bool					IsUploading() { return bIsUploading; }
	static bool					IsDownloading() { return bIsDownloading; }

private:

	static UE_API ERemoteCacheState	RemoteCacheState;
	static UE_API FText				RemoteCacheWarningMessage;
	static UE_API double				LastGetTime;
	static UE_API double				LastPutTime;
	static UE_API bool					bIsUploading;
	static UE_API bool					bIsDownloading;

};

#undef UE_API
