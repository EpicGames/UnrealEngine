// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API DERIVEDDATAWIDGETS_API

class SWidget;

class SDerivedDataRemoteStoreDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDerivedDataRemoteStoreDialog) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:

	UE_API TSharedRef<SWidget> GetGridPanel();

	UE_API EActiveTimerReturnType UpdateGridPanels(double InCurrentTime, float InDeltaTime);

	SVerticalBox::FSlot* GridSlot = nullptr;
};

class SDerivedDataCacheStatisticsDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDerivedDataCacheStatisticsDialog) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:

	UE_API TSharedRef<SWidget> GetGridPanel();
	
	UE_API EActiveTimerReturnType UpdateGridPanels(double InCurrentTime, float InDeltaTime);

	SVerticalBox::FSlot* GridSlot = nullptr;
};

class SDerivedDataResourceUsageDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDerivedDataResourceUsageDialog) {}
	SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

private:

	UE_API TSharedRef<SWidget> GetGridPanel();
	
	UE_API EActiveTimerReturnType UpdateGridPanels(double InCurrentTime, float InDeltaTime);

	SVerticalBox::FSlot* GridSlot = nullptr;
};

#undef UE_API
